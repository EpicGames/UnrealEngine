[Configuration]
BasedOn=..\Engine\Config\BaseEngine.ini

[Core.Log]
LogInit=warning
LogTaskGraph=warning
LogDevObjectVersion=warning
LogMemory=warning
LogTextLocalizationManager=warning
LogObj=warning
LogExit=warning
LogPlatformFile=warning
LogICUInternationalization=warning
LogHttp=error
LogOnline=error

[BuildPatchTool.ConsoleLog]
+ExcludedCategories=LogOnline
+ExcludedCategories=LogHttp

[OnlineSubsystem]
DefaultPlatformService=Mcp

[OnlineSubsystemMcp]
bEnabled=true

[HTTP]
HttpMaxConnectionsPerServer=0

; OnlinePartyMcp is not needed, but spews warnings if you do not provide default config
[OnlineSubsystemMcp.OnlinePartyMcp]
PartyPendingCreateResponseTimeout=0
PartyPendingJoinResponseTimeout=20
PartyPendingLeaveResponseTimeout=10
PartyPendingKickResponseTimeout=10
PartyPendingJoinRequestTimeout=20

;========== ACCOUNT SERVICE ==========
[OnlineSubsystemMcp.OnlineIdentityMcp]
ServiceName=account
Protocol=https

[OnlineSubsystemMcp.OnlineIdentityMcp Prod]
Domain=account-public-service-prod03.ol.epicgames.com
RedirectUrl=accounts.epicgames.com

[OnlineSubsystemMcp.OnlineIdentityMcp Stage]
Domain=account-public-service-stage.ol.epicgames.com
RedirectUrl=accounts.epicgames-stage.ol.epicgames.com

[OnlineSubsystemMcp.OnlineIdentityMcp GameDev]
Domain=account-public-service-common-gamedev.ol.epicgames.net
RedirectUrl=accounts.epicgames-gamedev.ol.epicgames.net

[OnlineSubsystemMcp.OnlineIdentityMcp LocalHost]
Domain=account-public-service-common-gamedev.ol.epicgames.net
RedirectUrl=accounts.epicgames-gamedev.ol.epicgames.net

[OnlineSubsystemMcp.OnlineIdentityMcp CI]
Domain=account-public-service-common-ci.ol.epicgames.net
RedirectUrl=accounts.epicgames-ci.ol.epicgames.net


;========== ARTIFACT SERVICE ==========
[OnlineSubsystemMcp.OnlineArtifactServiceMcp]
ServiceName=artifact-service
Protocol=https

[OnlineSubsystemMcp.OnlineArtifactServiceMcp Prod]
Domain=artifact-public-service-prod.ol.epicgames.com

[OnlineSubsystemMcp.OnlineArtifactServiceMcp Stage]
Domain=artifact-public-service-stage.ol.epicgames.com

[OnlineSubsystemMcp.OnlineArtifactServiceMcp GameDev]
Domain=artifact-public-service-gamedev.ol.epicgames.net

[OnlineSubsystemMcp.OnlineArtifactServiceMcp CI]
Domain=artifact-public-service-ci.ol.epicgames.net

[OnlineSubsystemMcp.OnlineArtifactServiceMcp LocalHost]
Domain=artifact-public-service-gamedev.ol.epicgames.net

;========== ARTIFACT DELIVERY SERVICE ==========
[OnlineSubsystemMcp.OnlineArtifactDeliveryServiceMcp]
ServiceName=artifact-delivery
Protocol=https

[OnlineSubsystemMcp.OnlineArtifactDeliveryServiceMcp GameDev]
Domain=artifact-delivery-public-service-gamedev.aeee.dev.use1a.on.epicgames.com

[OnlineSubsystemMcp.OnlineArtifactDeliveryServiceMcp Prod]
Domain=artifact-delivery-service-public-prod.ol.epicgames.com
+AltDomains=artifact-delivery-public-service-prod.beee.live.use1a.on.epicgames.com

;========== LAUNCHER SERVICE ==========
[OnlineSubsystemMcp.OnlineBuildInfoMcpV2]
ServiceName=launcher
Protocol=https

[OnlineSubsystemMcp.OnlineBuildInfoMcpV2 Prod]
Domain=launcher-public-service-prod06.ol.epicgames.com

[OnlineSubsystemMcp.OnlineBuildInfoMcpV2 Stage]
Domain=launcher-public-service-stage.ol.epicgames.

[OnlineSubsystemMcp.OnlineBuildInfoMcpV2 GameDev]
Domain=launcher-public-service-gamedev.ol.epicgames.net

[OnlineSubsystemMcp.OnlineBuildInfoMcpV2 CI]
Domain=launcher-public-service-ci.ol.epicgames.net

[OnlineSubsystemMcp.OnlineBuildInfoMcpV2 LocalHost]
Domain=launcher-public-service-gamedev.ol.epicgames.net

;========== CONTENT PROTECTION SERVICE ==========

[OnlineSubsystemMcp.ContentProtectionServiceMcp]
ServiceName=contentprotection
Protocol=https

[OnlineSubsystemMcp.ContentProtectionServiceMcp GameDev]
Domain=content-protection-service-gamedev.dev.use1a.on.epicgames.net
